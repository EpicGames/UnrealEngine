// Copyright Epic Games, Inc. All Rights Reserved.

#include "MkvFileReader.h"

#if WITH_WEBM_LIBS

#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

bool FMkvFileReader::Open(const TCHAR* Filename)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	m_file.Reset(PlatformFile.OpenRead(Filename));
	return m_file.IsValid();
}

FTimespan FMkvFileReader::GetVideoFrameDuration(const mkvparser::VideoTrack& track)
{
	double FrameRate = track.GetFrameRate();
	if (FrameRate > 0)
	{
		return FTimespan::FromSeconds(1.0 / FrameRate);
	}
	else
	{
		return FTimespan::FromSeconds(1.0 / 30.0);
	}
}

int FMkvFileReader::Read(long long Position, long Lenght, unsigned char* Buffer)
{
	check(m_file.IsValid());

	m_file->Seek(Position);
	m_file->Read(Buffer, Lenght);
	return 0 /* success */;
}

int FMkvFileReader::Length(long long* Total, long long* Available)
{
	check(m_file.IsValid());

	int64 CurrentPosition = m_file->Tell();

	m_file->SeekFromEnd(0);
	int64 FileSize = m_file->Tell();

	m_file->Seek(CurrentPosition);

	// Don't know the difference between Total and Available, but below code is required to get MkvParser going

	if (Total)
	{
		*Total = FileSize;
	}

	if (Available)
	{
		*Available = FileSize;
	}

	return 0 /* success */;
}

#endif // WITH_WEBM_LIBS
