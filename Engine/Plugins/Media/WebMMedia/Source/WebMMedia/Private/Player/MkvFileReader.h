// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBM_LIBS

#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

THIRD_PARTY_INCLUDES_START
#include "mkvparser.hpp"
THIRD_PARTY_INCLUDES_END

class FMkvFileReader : public mkvparser::IMkvReader
{
public:
	bool Open(const TCHAR* Filename);

	static FTimespan GetVideoFrameDuration(const mkvparser::VideoTrack& track);

public:
	// IMkvReader interface
	virtual int Read(long long Position, long Lenght, unsigned char* Buffer) override;
	virtual int Length(long long* Total, long long* Available) override;

private:
	TUniquePtr<IFileHandle> m_file;
};

#endif // WITH_WEBM_LIBS
