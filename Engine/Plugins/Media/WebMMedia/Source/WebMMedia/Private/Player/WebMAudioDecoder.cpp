// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMAudioDecoder.h"
#include "WebMMediaPrivate.h"
#include "WebMMediaFrame.h"
#include "WebMMediaAudioSample.h"
#include "WebMSamplesSink.h"

THIRD_PARTY_INCLUDES_START
#include <opus.h>
#include <vorbis/codec.h>
THIRD_PARTY_INCLUDES_END

struct FWebMAudioDecoder::FVorbisDecoder
{
	vorbis_info Info;
	vorbis_dsp_state DspState;
	vorbis_block Block;
};

FWebMAudioDecoder::FWebMAudioDecoder(IWebMSamplesSink& InSamples)
	: AudioSamplePool(new FWebMMediaAudioSamplePool)
	, Samples(InSamples)
	, OpusDecoder(nullptr)
	, CurrentSequenceIndex(0)
{
}

FWebMAudioDecoder::~FWebMAudioDecoder()
{
	if (AudioDecodingTask && !AudioDecodingTask->IsComplete())
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(AudioDecodingTask);
	}

	if (OpusDecoder)
	{
		opus_decoder_destroy(OpusDecoder);
	}
}

bool FWebMAudioDecoder::Initialize(const char* InCodec, int32 InSampleRate, int32 InChannels, const uint8* CodecPrivateData, size_t CodecPrivateDataSize)
{
	return InitializeInternal(FString(InCodec), InSampleRate, InChannels, MakeConstArrayView<uint8>(CodecPrivateData, (int32) CodecPrivateDataSize));
}

bool FWebMAudioDecoder::InitializeInternal(FString InCodec, int32 InSampleRate, int32 InChannels, TConstArrayView<uint8> InCodecPrivate)
{
	CodecSpecificData = InCodecPrivate;

	if (OpusDecoder)
	{
		opus_decoder_destroy(OpusDecoder);
		OpusDecoder = nullptr;
	}

	if (InCodec.Equals(TEXT("A_OPUS")))
	{
		Codec = ECodec::Opus;
	}
	else if (InCodec.Equals(TEXT("A_VORBIS")))
	{
		Codec = ECodec::Vorbis;
	}
	else
	{
		Codec = ECodec::Undefined;
		return false;
	}

	SampleRate = InSampleRate;
	Channels = InChannels;

	if (Codec == ECodec::Opus)
	{
		InitializeOpus();
	}
	else
	{
		if (!InitializeVorbis(CodecSpecificData.GetData(), CodecSpecificData.Num()))
		{
			return false;
		}
	}

	DecodeBuffer.SetNumUninitialized(FrameSize * Channels * sizeof(int16));
	return true;
}

void FWebMAudioDecoder::InitializeOpus()
{
	int32 ErrorCode = 0;
	OpusDecoder = opus_decoder_create(SampleRate, Channels, &ErrorCode);
	check(OpusDecoder);
	check(ErrorCode == 0);

	// Max supported frame is 120ms
	FrameSize = 120 * SampleRate / 1000;
}

bool FWebMAudioDecoder::InitializeVorbis(const uint8* CodecPrivateData, size_t CodecPrivateDataSize)
{
	if (CodecPrivateDataSize < 3 || !CodecPrivateData || CodecPrivateData[0] != 2)
	{
		UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis - invalid data");
		return false;
	}

	size_t HeaderSize[3] = { 0 };
	size_t Offset = 1;

	// Calculate three headers sizes
	for(int32 i = 0; i < 2; ++i)
	{
		for(;;)
		{
			if (Offset >= CodecPrivateDataSize)
			{
				UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis - invalid offset");
				return false;
			}
			HeaderSize[i] += CodecPrivateData[Offset];
			if (CodecPrivateData[Offset++] < 0xFF)
			{
				break;
			}
		}
	}
	HeaderSize[2] = CodecPrivateDataSize - (HeaderSize[0] + HeaderSize[1] + Offset);

	if (HeaderSize[0] + HeaderSize[1] + HeaderSize[2] + Offset != CodecPrivateDataSize)
	{
		UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis - parameters mismatch");
		return false;
	}

	ogg_packet Packet[3];
	memset(Packet, 0, 3 * sizeof(ogg_packet));

	Packet[0].packet = (unsigned char *) CodecPrivateData + Offset;
	Packet[0].bytes = HeaderSize[0];
	Packet[0].b_o_s = 1;

	Packet[1].packet = (unsigned char *) CodecPrivateData + Offset + HeaderSize[0];
	Packet[1].bytes = HeaderSize[1];

	Packet[2].packet = (unsigned char *) CodecPrivateData + Offset + HeaderSize[0] + HeaderSize[1];
	Packet[2].bytes = HeaderSize[2];

	VorbisDecoder.Reset(new FVorbisDecoder());

	vorbis_info_init(&VorbisDecoder->Info);

	/* Upload three Vorbis headers into libvorbis */
	vorbis_comment Vc;
	vorbis_comment_init(&Vc);
	for(int i = 0; i < 3; ++i)
	{
		if (vorbis_synthesis_headerin(&VorbisDecoder->Info, &Vc, &Packet[i]))
		{
			UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis - invalid headers");
			vorbis_comment_clear(&Vc);
			return false;
		}
	}

	vorbis_comment_clear(&Vc);

	if (vorbis_synthesis_init(&VorbisDecoder->DspState, &VorbisDecoder->Info))
	{
		UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis (synthesis)");
		return false;
	}

	if (VorbisDecoder->Info.channels != Channels || VorbisDecoder->Info.rate != SampleRate)
	{
		UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis - invalid parameters");
		return false;
	}

	if (vorbis_block_init(&VorbisDecoder->DspState, &VorbisDecoder->Block))
	{
		UE_LOGF(LogWebMMedia, Warning, "Failed to initialize Vorbis");
		return false;
	}

	FrameSize = 4096 / Channels;

	return true;
}

void FWebMAudioDecoder::DecodeAudioFramesAsync(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& AudioFrames)
{
	FGraphEventRef PreviousDecodingTask = AudioDecodingTask;

	AudioDecodingTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this, PreviousDecodingTask, AudioFrames]()
	{
		if (PreviousDecodingTask && !PreviousDecodingTask->IsComplete())
		{
			FTaskGraphInterface::Get().WaitUntilTaskCompletes(PreviousDecodingTask);
		}

		DoDecodeAudioFrames(AudioFrames);
	}, TStatId(), nullptr, ENamedThreads::AnyThread);
}

bool FWebMAudioDecoder::IsBusy() const
{
	return AudioDecodingTask && !AudioDecodingTask->IsComplete();
}

void FWebMAudioDecoder::DoDecodeAudioFrames(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& AudioFrames)
{
	for(const TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>& AudioFrame : AudioFrames)
	{
		int32 NumOfSamplesDecoded;

		// Change in codec?
		if (Codec == ECodec::Undefined)
		{
			if (AudioFrame->CodecName.IsEmpty())
			{
				UE_LOGF(LogWebMMedia, Display, "No codec specified to initialize the codec lazily");
				Samples.ReportAudioDecodingError(TEXT("No codec specified to initialize the codec lazily"));
				return;
			}
		}
		if (AudioFrame->CodecName.Len())
		{
			if (!InitializeInternal(AudioFrame->CodecName, AudioFrame->SamplingRate, AudioFrame->NumChannels, AudioFrame->CodecSpecificData))
			{
				UE_LOGF(LogWebMMedia, Display, "Failed to create codec lazily");
				Samples.ReportAudioDecodingError(TEXT("Failed to create codec lazily"));
				return;
			}
		}

		if (Codec == ECodec::Opus)
		{
			NumOfSamplesDecoded = DecodeOpus(AudioFrame);
		}
		else
		{
			NumOfSamplesDecoded = DecodeVorbis(AudioFrame);
		}

		if (NumOfSamplesDecoded > 0)
		{
			TSharedRef<FWebMMediaAudioSample, ESPMode::ThreadSafe> AudioSample = AudioSamplePool->AcquireShared();
			FTimespan Duration = (NumOfSamplesDecoded * ETimespan::TicksPerSecond) / SampleRate;
			AudioSample->Initialize(DecodeBuffer.GetData(), NumOfSamplesDecoded * Channels * 2, Channels, SampleRate, FMediaTimeStamp(AudioFrame->Time, AudioFrame->SequenceIndex), Duration, AudioFrame->DecoderIndex);

			Samples.AddAudioSampleFromDecodingThread(AudioSample);
		}
		else if (NumOfSamplesDecoded < 0)
		{
			Samples.ReportAudioDecodingError(TEXT("Error decoding audio frame"));
		}
	}
}

int32 FWebMAudioDecoder::DecodeOpus(const TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>& AudioFrame)
{
	// Change in sequence?
	if (AudioFrame->SequenceIndex != CurrentSequenceIndex)
	{
		opus_decoder_destroy(OpusDecoder);
		InitializeOpus();
		CurrentSequenceIndex = AudioFrame->SequenceIndex;
	}

	int32 Result = opus_decode(OpusDecoder, AudioFrame->Data.GetData(), AudioFrame->Data.Num(), (opus_int16*) DecodeBuffer.GetData(), FrameSize, 0);
	if (Result < 0)
	{
		UE_LOGF(LogWebMMedia, Warning, "Error decoding Opus audio frame");
	}

	return Result;
}

int32 FWebMAudioDecoder::DecodeVorbis(const TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>& AudioFrame)
{
	// Change in sequence?
	if (AudioFrame->SequenceIndex != CurrentSequenceIndex)
	{
		InitializeVorbis(CodecSpecificData.GetData(), CodecSpecificData.Num());
		CurrentSequenceIndex = AudioFrame->SequenceIndex;
	}

	ogg_packet Packet;
	memset(&Packet, 0, sizeof(ogg_packet));

	Packet.packet = AudioFrame->Data.GetData();
	Packet.bytes = AudioFrame->Data.Num();

	if (vorbis_synthesis(&VorbisDecoder->Block, &Packet))
	{
		UE_LOGF(LogWebMMedia, Warning, "Error decoding Vorbis audio frame - vorbis_synthesis failed");
		return -1;
	}

	if (vorbis_synthesis_blockin(&VorbisDecoder->DspState, &VorbisDecoder->Block))
	{
		UE_LOGF(LogWebMMedia, Warning, "Error decoding Vorbis audio frame - vorbis_synthesis_blockin failed");
		return -1;
	}

	int16* Buffer = (int16*) DecodeBuffer.GetData();
	int32 Count = 0;
	float** Pcm;
	int32 SamplesCount = vorbis_synthesis_pcmout(&VorbisDecoder->DspState, &Pcm);
	while (SamplesCount > 0)
	{
		int32 ToConvert = SamplesCount <= FrameSize ? SamplesCount : FrameSize;
		for(int32 Channel = 0; Channel < Channels; ++Channel)
		{
			float *FloatSamples = Pcm[Channel];
			for(int32 i = 0, j = Channel; i < ToConvert; ++i, j += Channels)
			{
				int32 Sample = FMath::Clamp<int32>(FloatSamples[i] * 32767.0f, -32768, 32767);
				Buffer[Count + j] = Sample;
			}
		}
		vorbis_synthesis_read(&VorbisDecoder->DspState, ToConvert);

		Count += ToConvert;

		SamplesCount = vorbis_synthesis_pcmout(&VorbisDecoder->DspState, &Pcm);
	}

	return Count;
}
