// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMMediaPlayer.h"

#if WITH_WEBM_LIBS

#include "WebMMediaPrivate.h"

#include "IMediaEventSink.h"
#include "MediaSamples.h"

#include "WebMVideoDecoder.h"
#include "WebMAudioDecoder.h"
#include "WebMMediaFrame.h"
#include "WebMMediaTextureSample.h"
#include "WebMMediaAudioSample.h"

#define LOCTEXT_NAMESPACE "FWebMMediaPlayer"

FWebMMediaPlayer::FWebMMediaPlayer(IMediaEventSink& InEventSink)
	: EventSink(InEventSink)
	, Samples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
{
}

FWebMMediaPlayer::~FWebMMediaPlayer()
{
	Close();
}

FWebMMediaPlayer::ETrackCodec FWebMMediaPlayer::GetCodecFromMKVTrack(const char* InMKVCodecName)
{
	if (FCStringAnsi::Strcmp(InMKVCodecName, "V_VP8") == 0)
	{
		return ETrackCodec::VP8;
	}
	else if (FCStringAnsi::Strcmp(InMKVCodecName, "V_VP9") == 0)
	{
		return ETrackCodec::VP9;
	}
	else if (FCStringAnsi::Strcmp(InMKVCodecName, "A_OPUS") == 0)
	{
		return ETrackCodec::Opus;
	}
	else if (FCStringAnsi::Strcmp(InMKVCodecName, "A_VORBIS") == 0)
	{
		return ETrackCodec::Vorbis;
	}
	return ETrackCodec::Undefined;
}
FString FWebMMediaPlayer::GetMKVCodec(ETrackCodec InCodec)
{
	switch(InCodec)
	{
		case ETrackCodec::VP8:
		{
			return FString(TEXT("V_VP8"));
		}
		case ETrackCodec::VP9:
		{
			return FString(TEXT("V_VP9"));
		}
		case ETrackCodec::Opus:
		{
			return FString(TEXT("A_OPUS"));
		}
		case ETrackCodec::Vorbis:
		{
			return FString(TEXT("A_VORBIS"));
		}
		default:
		{
			return FString(TEXT("X_UNKNOWN"));
		}
	}
}


void FWebMMediaPlayer::Close()
{
	VideoDecoder.Reset();
	AudioDecoder.Reset();
	VideoTracks.Empty();
	AudioTracks.Empty();
	MkvReader.Reset();
	MkvSegment.Reset();
	MkvCurrentCluster = nullptr;
	MkvCurrentBlockEntry = nullptr;
	MediaUrl.Empty();
	SelectedAudioTrack = INDEX_NONE;
	SelectedVideoTrack = INDEX_NONE;
	NewPendingVideoCodec.Reset();
	NewPendingAudioCodec.Reset();
	bPendingVideoTrackChange = false;
	bPendingAudioTrackChange = false;
	CurrentSeekIndex = 0;
	CurrentLoopIndex = 0;
	CurrentTime = FTimespan::Zero();
	SeekToTime.Reset();
	SeekTo.Reset();
	CurrentVideoTime.Reset();
	CurrentAudioTime.Reset();
	bFileReachedEOS = false;
	bFileError = false;
	bDecodeError = false;
	bLooping = false;

	VideoFramesReadyForDecode.Empty();
	AudioFramesReadyForDecode.Empty();
	LoadedVideoFrames.Empty();
	LoadedAudioFrames.Empty();
	MovieDuration = FTimespan::Zero();
	VideoFrameRate.Numerator = 0;
	VideoFrameRate.Denominator = 0;
	bRecalculateFrameRate = false;

	VideoFrameDoneDelegate.Unbind();
	AudioFrameDoneDelegate.Unbind();

	DecodedSamplesLock.Lock();
	NumDecodingEnqueuedVideoFrames = 0;
	NumDecodingEnqueuedAudioFrames = 0;
	DecodedVideoFrames.Empty();
	DecodedAudioFrames.Empty();
	DecodedSamplesLock.Unlock();

	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> NewSamples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>());
	MediaSamplesLock.Lock();
	Swap(Samples, NewSamples);
	MediaSamplesLock.Unlock();
	NewSamples.Reset();

	if (CurrentState != EMediaState::Closed)
	{
		CurrentState = EMediaState::Closed;
		// notify listeners
		OutEvents.Push(EMediaEvent::TracksChanged);
		OutEvents.Push(EMediaEvent::MediaClosed);
	}
}

IMediaCache& FWebMMediaPlayer::GetCache()
{
	return *this;
}

IMediaControls& FWebMMediaPlayer::GetControls()
{
	return *this;
}

FString FWebMMediaPlayer::GetInfo() const
{
	return TEXT("WebMMedia information not implemented yet");
}

FGuid FWebMMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0xdfbb4e57, 0x07dc4b4a, 0xa25b5cba, 0x0f963ac3);
	return PlayerPluginGUID;
}

IMediaSamples& FWebMMediaPlayer::GetSamples()
{
	FScopeLock lock(&MediaSamplesLock);
	return *Samples.Get();
}

FString FWebMMediaPlayer::GetStats() const
{
	return TEXT("WebMMedia stats information not implemented yet");
}

IMediaTracks& FWebMMediaPlayer::GetTracks()
{
	return *this;
}

FString FWebMMediaPlayer::GetUrl() const
{
	return MediaUrl;
}

IMediaView& FWebMMediaPlayer::GetView()
{
	return *this;
}

bool FWebMMediaPlayer::Open(const FString& Url, const IMediaOptions* /*Options*/)
{
	if (CurrentState == EMediaState::Error)
	{
		return false;
	}

	Close();

	if ((Url.IsEmpty()))
	{
		return false;
	}

	MediaUrl = Url;

	// open the media
	if (!Url.StartsWith(TEXT("file://")))
	{
		UE_LOGF(LogWebMMedia, Error, "Not supported URL: %ls", *Url);
		return false;
	}

	FString FilePath = Url.RightChop(7);
	FPaths::NormalizeFilename(FilePath);

	MkvReader.Reset(new FMkvFileReader());
	if (!MkvReader->Open(*FilePath))
	{
		UE_LOGF(LogWebMMedia, Error, "Failed opening video file: %ls", *FilePath);
		return false;
	}

	if (!MkvRead())
	{
		MkvReader.Reset();

		UE_LOGF(LogWebMMedia, Error, "Error parsing matroska file: %ls", *FilePath);
		return false;
	}

	VideoDecoder.Reset(new FWebMVideoDecoder(*this));
	AudioDecoder.Reset(new FWebMAudioDecoder(*this));
	CurrentState = EMediaState::Stopped;
	bFileReachedEOS = false;
	bFileError = false;
	bDecodeError = false;

	// notify listeners
	OutEvents.Push(EMediaEvent::TracksChanged);
	OutEvents.Push(EMediaEvent::MediaOpened);

	VideoFrameDoneDelegate.BindRaw(this, &FWebMMediaPlayer::ReturnVideoSample);
	AudioFrameDoneDelegate.BindRaw(this, &FWebMMediaPlayer::ReturnAudioSample);

	return true;
}

bool FWebMMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& /*Archive*/, const FString& /*OriginalUrl*/, const IMediaOptions* /*Options*/)
{
	// We do not support opening archives for now.
	return false;
}

void FWebMMediaPlayer::TickInternal()
{
	// Load future data unless in a state where it is not possible or does not make sense.
	if (CurrentState == EMediaState::Closed ||
		CurrentState == EMediaState::Error ||
		CurrentState == EMediaState::Stopped ||
		bFileError || bDecodeError)
	{
		return;
	}

	// Is a seek pending?
	bool bSeekPending = SeekToTime.IsSet();
	if (bSeekPending)
	{
		CurrentTime = SeekToTime.GetValue();
		EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);
	}

	// Is there a pending seek or track change?
	if (bSeekPending || bPendingVideoTrackChange || bPendingAudioTrackChange)
	{
		// Clamp time if necessary
		if (CurrentTime < FTimespan::Zero())
		{
			CurrentTime = FTimespan::Zero();
		}
		else if (CurrentTime > MovieDuration)
		{
			CurrentTime = MovieDuration;
		}

		bool bDeselectVideo = bPendingVideoTrackChange && SelectedVideoTrack == INDEX_NONE;
		bool bDeselectAudio = bPendingAudioTrackChange && SelectedAudioTrack == INDEX_NONE;
		bool bDeselectOnly = !bSeekPending &&
							 ((bPendingVideoTrackChange && bDeselectVideo && !bPendingAudioTrackChange) ||
							  (bPendingAudioTrackChange && bDeselectAudio && !bPendingVideoTrackChange) ||
							  (bDeselectVideo && bDeselectAudio));

		if (!bDeselectOnly)
		{
			// A track change is similar to a seek in that we have to restart at a keyframe as well.
			SeekTo.Reset();
			SeekTo.TargetTime = CurrentTime;
			SeekTo.bAwaitTime = true;
			SeekToTime.Reset();
			CurrentVideoTime.Reset();
			CurrentAudioTime.Reset();

			// We need to fetch data for different tracks or at a different time now, so dump everything we have going at the moment.
			VideoFramesReadyForDecode.Empty();
			AudioFramesReadyForDecode.Empty();
			LoadedVideoFrames.Empty();
			LoadedAudioFrames.Empty();
			Samples->FlushSamples();
			if (bPendingVideoTrackChange && SelectedVideoTrack != INDEX_NONE)
			{
				NewPendingVideoCodec = GetCodecFromMKVTrack(VideoTracks[SelectedVideoTrack]->GetCodecId());
			}
			if (bPendingAudioTrackChange && SelectedAudioTrack != INDEX_NONE)
			{
				NewPendingAudioCodec = GetCodecFromMKVTrack(AudioTracks[SelectedAudioTrack]->GetCodecId());
			}
			MkvSeekToTime(CurrentTime);
			bFileReachedEOS = false;
			++CurrentDecoderIndex;
		}
		else
		{
			if (bDeselectVideo)
			{
				VideoFramesReadyForDecode.Empty();
				LoadedVideoFrames.Empty();
				/*
					TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
					TRange<FTimespan> TimeRange(FTimespan::Zero(), FTimespan::MaxValue());
					Samples->FetchVideo(TimeRange, VideoSample);
				*/
			}
			if (bDeselectAudio)
			{
				AudioFramesReadyForDecode.Empty();
				LoadedAudioFrames.Empty();
				/*
					TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe> AudioSample;
					TRange<FTimespan> TimeRange(FTimespan::Zero(), FTimespan::MaxValue());
					Samples->FetchAudio(TimeRange, AudioSample);
				*/
			}
		}
		bPendingVideoTrackChange = false;
		bPendingAudioTrackChange = false;
	}

	// If no track is selected we do not need to load anything
	if (SelectedVideoTrack == INDEX_NONE && SelectedAudioTrack == INDEX_NONE)
	{
		return;
	}

	if (bFileReachedEOS)
	{
		if (bLooping)
		{
			MkvCurrentCluster = MkvSegment->FindCluster(0);
			MkvCurrentBlockEntry = nullptr;
			++CurrentLoopIndex;
			bFileReachedEOS = false;
		}
		else
		{
			return;
		}
	}

	// Number of future video frames we want to have, regardless of framerate.
	int32 kNumFutureVideoFrames = VideoFrameRate.IsValid() ? (int32) VideoFrameRate.AsDecimal() : 30;
	kNumFutureVideoFrames = kNumFutureVideoFrames > 60 ? 60 : kNumFutureVideoFrames < 5 ? 5 : kNumFutureVideoFrames;
	const int32 kNumFutureAudioFrames = 60;

	int64 CurrentSequenceIndex = FMediaTimeStamp::MakeIndexValue(CurrentSeekIndex, CurrentLoopIndex);

	// Read frames up to 1 secs in the future
	while(1)
	{
		bool bHaveEnough = true;
		const bool bNeedVideo = SelectedVideoTrack != INDEX_NONE;
		const bool bNeedAudio = SelectedAudioTrack != INDEX_NONE;
		if (bNeedVideo)
		{
			bHaveEnough = (VideoFramesReadyForDecode.Num() + LoadedVideoFrames.Num()) >= kNumFutureVideoFrames;
		}
		else if (bNeedAudio)
		{
			bHaveEnough = (AudioFramesReadyForDecode.Num() + LoadedAudioFrames.Num()) >= kNumFutureAudioFrames;
		}
		if (bHaveEnough)
		{
			break;
		}

		MkvSeekToNextValidBlock();

		if (bFileReachedEOS || bFileError)
		{
			break;
		}

		auto Blk = MkvCurrentBlockEntry->GetBlock();
		const auto TrackNumber = Blk->GetTrackNumber();
		const bool bIsVideo = bNeedVideo && VideoTracks[SelectedVideoTrack]->GetNumber() == TrackNumber;
		const bool bIsAudio = bNeedAudio && AudioTracks[SelectedAudioTrack]->GetNumber() == TrackNumber;
		if (!bIsVideo && !bIsAudio)
		{
			continue;
		}

		FTimespan Timestamp(MkvCurrentBlockEntry->GetBlock()->GetTime(MkvCurrentCluster) / 100);
		const bool bIsKeyframe = MkvCurrentBlockEntry->GetBlock()->IsKey();
		const bool bIsInvisible = MkvCurrentBlockEntry->GetBlock()->IsInvisible();
		check(!bIsInvisible);

		// Wait for target time to appear?
		if (SeekTo.bAwaitTime)
		{
			// We need a video keyframe
			if (bIsVideo)
			{
				if (bIsKeyframe)
				{
					SeekTo.bGotVideoKeyframe = true;
					SeekTo.TargetTime = Timestamp;
					CurrentTime = Timestamp;
				}
				else if (!SeekTo.bGotVideoKeyframe)
				{
					continue;
				}
			}
			// If this is an audio sample and we need video but haven't found a video keyframe yet
			// then this audio sample is not one we want.
			else if (bIsAudio && bNeedVideo && !SeekTo.bGotVideoKeyframe)
			{
				continue;
			}
			// Anything that is before the time we want we do not need.
			if (Timestamp < SeekTo.TargetTime)
			{
				continue;
			}
			else
			{
				SeekTo.Reset();
			}
		}

		const int32 FrameCount = (int32) MkvCurrentBlockEntry->GetBlock()->GetFrameCount();
		for(int32 i=0; i<FrameCount; ++i)
		{
			const mkvparser::Block::Frame& MkvFrame = MkvCurrentBlockEntry->GetBlock()->GetFrame(i);

			TSharedPtr<FWebMFrame, ESPMode::ThreadSafe> Frame = MakeShared<FWebMFrame>();
			Frame->Time = i==0 ? Timestamp : FTimespan::MinValue();
			Frame->SequenceIndex = CurrentSequenceIndex;
			Frame->DecoderIndex = CurrentDecoderIndex;
			Frame->bIsKeyframe = bIsKeyframe;

			Frame->Data.SetNumUninitialized(MkvFrame.len);
			bool bReadOk = 0 == MkvFrame.Read(MkvReader.Get(), Frame->Data.GetData());
			if (bIsVideo)
			{
				LoadedVideoFrames.Add(Frame);
			}
			else if (bIsAudio)
			{
				LoadedAudioFrames.Add(Frame);
			}
		}
	}

	auto FixDurationsAndTimestamps = [](TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& InFrames, FTimespan& InOutDurationSum, bool bIsFinalFrame, FTimespan InMovieDuration) -> int32
	{
		if (bIsFinalFrame)
		{
			TSharedPtr<FWebMFrame, ESPMode::ThreadSafe> Frame = MakeShared<FWebMFrame>();
			Frame->Time = InMovieDuration;
			InFrames.Emplace(MoveTemp(Frame));
		}

		int32 LastGoodSampleIndex = -1;
		for(int32 i=0, iMax=InFrames.Num(); i<iMax; ++i)
		{
			if (InFrames[i]->Duration == FTimespan::Zero())
			{
				// Look forward to where the next sample that has a timestamp is.
				for(int32 j=i+1; j<iMax; ++j)
				{
					if (InFrames[j]->Time > FTimespan::Zero())
					{
						int32 NumF = j - i;
						FTimespan DurationSpan = InFrames[j]->Time - InFrames[i]->Time;
						if (DurationSpan < FTimespan::Zero())
						{
							DurationSpan = FTimespan::Zero();
						}
						InOutDurationSum += DurationSpan;
						if (NumF == 1)
						{
							InFrames[i]->Duration = DurationSpan;
							LastGoodSampleIndex = i;
						}
						else
						{
							FTimespan DurPerSample = DurationSpan / NumF;
							for(int32 k=i; k<j-1; ++k)
							{
								InFrames[k]->Duration = DurPerSample;
								InFrames[k+1]->Time = InFrames[k]->Time + InFrames[k]->Duration;
							}
							InFrames[j-1]->Duration = InFrames[j]->Time - InFrames[j-1]->Time;
							i = j - 1;
							LastGoodSampleIndex = i;
						}
						break;
					}
				}
			}
		}
		if (bIsFinalFrame)
		{
			InFrames.Pop();
			LastGoodSampleIndex = InFrames.Num() - 1;
		}
		return LastGoodSampleIndex;
	};


	FTimespan DurationSumA, DurationSumV;
	int32 LastGoodVideoSampleIndex = FixDurationsAndTimestamps(LoadedVideoFrames, DurationSumV, bFileReachedEOS, MovieDuration);
	int32 LastGoodAudioSampleIndex = FixDurationsAndTimestamps(LoadedAudioFrames, DurationSumA, bFileReachedEOS, MovieDuration);
	if (bRecalculateFrameRate && LastGoodVideoSampleIndex > 1)
	{
		VideoFrameRate.Numerator = ETimespan::TicksPerSecond;
		VideoFrameRate.Denominator = DurationSumV.GetTicks() / (LastGoodVideoSampleIndex + 1);
		bRecalculateFrameRate = false;
	}

	// Move the valid samples from the load to the decode queue.
	if (LastGoodVideoSampleIndex >= 0)
	{
		for(int32 i=0; i<=LastGoodVideoSampleIndex; ++i)
		{
			VideoFramesReadyForDecode.Emplace(LoadedVideoFrames[0]);
			LoadedVideoFrames.RemoveAt(0);
		}
	}
	if (LastGoodAudioSampleIndex >= 0)
	{
		for(int32 i=0; i<=LastGoodAudioSampleIndex; ++i)
		{
			AudioFramesReadyForDecode.Emplace(LoadedAudioFrames[0]);
			LoadedAudioFrames.RemoveAt(0);
		}
	}
}

void FWebMMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	TickInternal();

	// Send out events
	for(const auto& Event : OutEvents)
	{
		EventSink.ReceiveMediaEvent(Event);
	}
	OutEvents.Empty();
	if (CurrentState == EMediaState::Error)
	{
		return;
	}


	// Enqueue frames for decoding.
	if (!bDecodeError)
	{
		const int32 kMaxPendingVideoFrames = 5;
		const int32 kMaxPendingAudioFrames = 10;

		if (NumDecodingEnqueuedVideoFrames < kMaxPendingVideoFrames)
		{
			int32 FramesNeeded = kMaxPendingVideoFrames - NumDecodingEnqueuedVideoFrames;
			TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> FramesForDecode;
			FScopeLock lock(&DecodedSamplesLock);
			for(int32 i=0; i<FramesNeeded && VideoFramesReadyForDecode.Num(); ++i)
			{
				FramesForDecode.Emplace(VideoFramesReadyForDecode[0]);	//-V767
				VideoFramesReadyForDecode.RemoveAt(0);
			}
			if (FramesForDecode.Num())
			{
				// Is there a change in codec?
				if (NewPendingVideoCodec.IsSet())
				{
					FramesForDecode[0]->CodecName = GetMKVCodec(NewPendingVideoCodec.GetValue());
					NewPendingVideoCodec.Reset();
				}
				NumDecodingEnqueuedVideoFrames += FramesForDecode.Num();
				VideoDecoder->DecodeVideoFramesAsync(FramesForDecode);
			}
		}

		if (NumDecodingEnqueuedAudioFrames < kMaxPendingAudioFrames)
		{
			int32 FramesNeeded = kMaxPendingAudioFrames - NumDecodingEnqueuedAudioFrames;
			TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> FramesForDecode;
			FScopeLock lock(&DecodedSamplesLock);
			for(int32 i=0; i<FramesNeeded && AudioFramesReadyForDecode.Num(); ++i)
			{
				FramesForDecode.Emplace(AudioFramesReadyForDecode[0]);	//-V767
				AudioFramesReadyForDecode.RemoveAt(0);
			}
			if (FramesForDecode.Num())
			{
				// Is there a change in codec?
				if (NewPendingAudioCodec.IsSet())
				{
					FramesForDecode[0]->CodecName = GetMKVCodec(NewPendingAudioCodec.GetValue());
					NewPendingAudioCodec.Reset();
					size_t CSDSize = 0;
					const uint8* CSD = AudioTracks[SelectedAudioTrack]->GetCodecPrivate(CSDSize);
					FramesForDecode[0]->CodecSpecificData = MakeConstArrayView<uint8>(CSD, (int32)CSDSize);
					FramesForDecode[0]->NumChannels = (int32) AudioTracks[SelectedAudioTrack]->GetChannels();
					FramesForDecode[0]->SamplingRate = (int32) AudioTracks[SelectedAudioTrack]->GetSamplingRate();
				}
				NumDecodingEnqueuedAudioFrames += FramesForDecode.Num();
				AudioDecoder->DecodeAudioFramesAsync(FramesForDecode);
			}
		}
	}

	// Pass decoded samples to the media sample queue.
	DecodedSamplesLock.Lock();
	while(Samples->CanReceiveVideoSamples(1) && DecodedVideoFrames.Num())
	{
		auto VideoFrame = DecodedVideoFrames[0];
		DecodedVideoFrames.RemoveAt(0);
		Samples->AddVideo(VideoFrame.ToSharedRef());
		--NumDecodingEnqueuedVideoFrames;
	}
	while(Samples->CanReceiveAudioSamples(1) && DecodedAudioFrames.Num())
	{
		auto AudioFrame = DecodedAudioFrames[0];
		DecodedAudioFrames.RemoveAt(0);
		Samples->AddAudio(AudioFrame.ToSharedRef());
		--NumDecodingEnqueuedAudioFrames;
	}
	DecodedSamplesLock.Unlock();

	// Encountered an error?
	if ((bFileError || bDecodeError) && CurrentState != EMediaState::Error)
	{
		CurrentState = EMediaState::Error;
		OutEvents.Push(EMediaEvent::PlaybackSuspended);
		LoadedVideoFrames.Empty();
		LoadedAudioFrames.Empty();
		VideoFramesReadyForDecode.Empty();
		AudioFramesReadyForDecode.Empty();
		return;
	}

	// Update current time with either the video or audio time
	if (CurrentState == EMediaState::Playing)
	{
		if (SelectedVideoTrack != INDEX_NONE && VideoTracks.Num())
		{
			FScopeLock lock(&TimeLock);
			if (CurrentVideoTime.IsSet())
			{
				CurrentTime = CurrentVideoTime.GetValue();
			}
		}
		else if (SelectedAudioTrack != INDEX_NONE && AudioTracks.Num())
		{
			FScopeLock lock(&TimeLock);
			if (CurrentAudioTime.IsSet())
			{
				CurrentTime = CurrentAudioTime.GetValue();
			}
		}
		else
		{
			CurrentTime += DeltaTime;
			if (CurrentTime > MovieDuration)
			{
				CurrentTime = MovieDuration;
			}
		}
	}

	// Reached the end of the file while reading?
	if (!bLooping && bFileReachedEOS && CurrentState == EMediaState::Playing)
	{
		// Check if we have processed all pending input.
		if (VideoFramesReadyForDecode.IsEmpty() && AudioFramesReadyForDecode.IsEmpty() &&
			LoadedVideoFrames.IsEmpty() && LoadedAudioFrames.IsEmpty())
		{
			CurrentTime = MovieDuration;
			CurrentState = EMediaState::Paused;
			OutEvents.Push(EMediaEvent::PlaybackEndReached);
			OutEvents.Push(EMediaEvent::PlaybackSuspended);
		}
	}
}

/* IMediaTracks interface
 *****************************************************************************/

bool FWebMMediaPlayer::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if (FormatIndex != 0)
	{
		// We only support one format per track
		return false;
	}

	if (TrackIndex == INDEX_NONE || TrackIndex >= AudioTracks.Num())
	{
		return false;
	}

	OutFormat.BitsPerSample = AudioTracks[TrackIndex]->GetBitDepth();
	OutFormat.NumChannels = AudioTracks[TrackIndex]->GetChannels();
	OutFormat.SampleRate = AudioTracks[TrackIndex]->GetSamplingRate();
	OutFormat.TypeName = UTF8_TO_TCHAR(AudioTracks[TrackIndex]->GetCodecNameAsUTF8());
	return true;
}

int32 FWebMMediaPlayer::GetNumTracks(EMediaTrackType TrackType) const
{
	if (TrackType == EMediaTrackType::Audio)
	{
		return AudioTracks.Num();
	}
	else if (TrackType == EMediaTrackType::Video)
	{
		return VideoTracks.Num();
	}
	else
	{
		// We support only video and audio tracks
		return 0;
	}
}

int32 FWebMMediaPlayer::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	// We only support one format per track
	return 1;
}

int32 FWebMMediaPlayer::GetSelectedTrack(EMediaTrackType TrackType) const
{
	if (TrackType == EMediaTrackType::Audio)
	{
		return SelectedAudioTrack;
	}
	else if (TrackType == EMediaTrackType::Video)
	{
		return SelectedVideoTrack;
	}
	else
	{
		return INDEX_NONE;
	}
}

FText FWebMMediaPlayer::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return FText::FromString(GetTrackName(TrackType, TrackIndex));
}

int32 FWebMMediaPlayer::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	// We only support one format per track
	return 0;
}

FString FWebMMediaPlayer::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	// TODO! We support only default language
	static FString Language(TEXT("Default"));
	return Language;
}

FString FWebMMediaPlayer::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	FString TrackName(TEXT("None"));

	if (TrackIndex != INDEX_NONE)
	{
		if (TrackType == EMediaTrackType::Audio && TrackIndex < AudioTracks.Num())
		{
			TrackName = AudioTracks[TrackIndex]->GetNameAsUTF8() ? UTF8_TO_TCHAR(AudioTracks[TrackIndex]->GetNameAsUTF8()) : FString::Printf(TEXT("Track %d"), TrackIndex);
		}
		else if (TrackType == EMediaTrackType::Video && TrackIndex < VideoTracks.Num())
		{
			TrackName = VideoTracks[TrackIndex]->GetNameAsUTF8() ? UTF8_TO_TCHAR(VideoTracks[TrackIndex]->GetNameAsUTF8()) : FString::Printf(TEXT("Track %d"), TrackIndex);
		}
	}

	return TrackName;
}

bool FWebMMediaPlayer::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (FormatIndex != 0)
	{
		// we support only one format
		return false;
	}

	if (TrackIndex == INDEX_NONE || TrackIndex >= VideoTracks.Num())
	{
		return false;
	}

	OutFormat.Dim = FIntPoint(VideoTracks[TrackIndex]->GetWidth(), VideoTracks[TrackIndex]->GetHeight());
	OutFormat.FrameRate = VideoFrameRate.IsValid() ? VideoFrameRate.AsDecimal() : 0.0f;
	OutFormat.TypeName = UTF8_TO_TCHAR(VideoTracks[TrackIndex]->GetCodecNameAsUTF8());
	OutFormat.FrameRates = TRange<float>{ OutFormat.FrameRate };
	return true;
}

bool FWebMMediaPlayer::SelectTrack(EMediaTrackType InTrackType, int32 InTrackIndex)
{
	if (InTrackType == EMediaTrackType::Video && InTrackIndex != SelectedVideoTrack && (InTrackIndex == INDEX_NONE || (InTrackIndex >= 0 && InTrackIndex < VideoTracks.Num())))
	{
		bPendingVideoTrackChange = true;
		SelectedVideoTrack = InTrackIndex;
		if (SelectedVideoTrack != INDEX_NONE)
		{
			const mkvparser::VideoTrack* VideoTrack = VideoTracks[SelectedVideoTrack];

			auto DefDurNanos = VideoTrack->GetDefaultDuration();
			if (DefDurNanos == 0)
			{
				UE_LOGF(LogWebMMedia, Error, "Video track default duration not present in file");
			}
			VideoFrameRate.Numerator = ETimespan::TicksPerSecond;
			VideoFrameRate.Denominator = DefDurNanos / 100;
			// Is the value reasonable? (less than 60fps)
			if (VideoFrameRate.IsValid() && VideoFrameRate.AsDecimal() > 60.5)
			{
				VideoFrameRate = FFrameRate(30,1);
				bRecalculateFrameRate = true;
			}
		}
		else
		{
			VideoFrameRate.Numerator = 0;
			VideoFrameRate.Denominator = 1;
		}
		return true;
	}
	else if (InTrackType == EMediaTrackType::Audio && InTrackIndex != SelectedAudioTrack && (InTrackIndex == INDEX_NONE || (InTrackIndex >= 0 && InTrackIndex < AudioTracks.Num())))
	{
		bPendingAudioTrackChange = true;
		SelectedAudioTrack = InTrackIndex;
		return true;
	}
	return false;
}

bool FWebMMediaPlayer::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	if (FormatIndex == 0)
	{
		return true;
	}
	else
	{
		// We only support one track format
		return false;
	}
}

/* IMediaControls interface
 *****************************************************************************/

bool FWebMMediaPlayer::CanControl(EMediaControl Control) const
{
	if (Control == EMediaControl::Pause)
	{
		return (CurrentState == EMediaState::Playing || CurrentState == EMediaState::Preparing);
	}
	else if (Control == EMediaControl::Resume)
	{
		return (CurrentState == EMediaState::Playing) || (CurrentState == EMediaState::Stopped);
	}
	else if (Control == EMediaControl::Seek)
	{
		return (CurrentState != EMediaState::Closed) && (CurrentState != EMediaState::Error);
	}
	return false;
}

FTimespan FWebMMediaPlayer::GetDuration() const
{
	return MovieDuration;
}

float FWebMMediaPlayer::GetRate() const
{
	return CurrentState == EMediaState::Playing ? 1.0f : 0.0f;
}

EMediaState FWebMMediaPlayer::GetState() const
{
	return CurrentState;
}

EMediaStatus FWebMMediaPlayer::GetStatus() const
{
	return CurrentState == EMediaState::Preparing ? EMediaStatus::Buffering : EMediaStatus::None;
}

TRangeSet<float> FWebMMediaPlayer::GetSupportedRates(EMediaRateThinning /*Thinning*/) const
{
	TRangeSet<float> Result;
	Result.Add(TRange<float>(0.0f));
	Result.Add(TRange<float>(1.0f));
	return Result;
}

FTimespan FWebMMediaPlayer::GetTime() const
{
	return CurrentTime;
}

bool FWebMMediaPlayer::IsLooping() const
{
	return bLooping;
}

bool FWebMMediaPlayer::Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams)
{
	if (CurrentState == EMediaState::Closed || CurrentState == EMediaState::Error)
	{
		UE_LOGF(LogWebMMedia, Warning, "Cannot seek while closed or in error state");
		return false;
	}
	SeekToTime = InNewTime;
	CurrentSeekIndex = InAdditionalParams.NewSequenceIndex.Get(0);
	return true;
}

bool FWebMMediaPlayer::SetLooping(bool Looping)
{
	bLooping = Looping;
	return true;
}

bool FWebMMediaPlayer::SetRate(float Rate)
{
	if (CurrentState == EMediaState::Closed || CurrentState == EMediaState::Error)
	{
		return false;
	}

	if (Rate == 0.0f)
	{
		Pause();
		return true;
	}
	else if (Rate == 1.0f)
	{
		Resume();
		return true;
	}
	else
	{
		return false;
	}
}

bool FWebMMediaPlayer::GetPlayerFeatureFlag(EFeatureFlag InFlag) const
{
	switch(InFlag)
	{
		case EFeatureFlag::UsePlaybackTimingV2:
		{
			return true;
		}
		case EFeatureFlag::PlayerUsesInternalFlushOnSeek:
		{
			return true;
		}
		default:
		{
			return IMediaPlayer::GetPlayerFeatureFlag(InFlag);
		}
	}
}

bool FWebMMediaPlayer::FlushOnSeekStarted() const
{
	return false;
}

bool FWebMMediaPlayer::FlushOnSeekCompleted() const
{
	return false;
}


bool FWebMMediaPlayer::MkvRead()
{
	int64 FilePosition = 0;
	if (mkvparser::EBMLHeader().Parse(MkvReader.Get(), FilePosition) != 0)
	{
		return false;
	}

	mkvparser::Segment* Segment;
	if (mkvparser::Segment::CreateInstance(MkvReader.Get(), FilePosition, Segment) != 0)
	{
		return false;
	}

	MkvSegment.Reset(Segment);

	if (MkvSegment->Load() < 0 || !MkvSegment->GetInfo())
	{
		return false;
	}

	MovieDuration = FTimespan::FromMicroseconds(MkvSegment->GetInfo()->GetDuration() / 1000);

	// Read all the tracks
	const mkvparser::Tracks* Tracks = MkvSegment->GetTracks();
	int32 NumOfTracks = Tracks->GetTracksCount();
	if (NumOfTracks == 0)
	{
		UE_LOGF(LogWebMMedia, Warning, "File doesn't have any tracks");
		return false;
	}

	for(int32 i=0; i<NumOfTracks; ++i)
	{
		const mkvparser::Track* Track = Tracks->GetTrackByIndex(i);
		check(Track);

		if (Track->GetType() == mkvparser::Track::kVideo)
		{
			if (GetCodecFromMKVTrack(Track->GetCodecId()) != ETrackCodec::Undefined)
			{
				VideoTracks.Add(static_cast<const mkvparser::VideoTrack*>(Track));
			}
			else
			{
				UE_LOGF(LogWebMMedia, Warning, "File contains unsupported video track %d: %ls", i, UTF8_TO_TCHAR(Track->GetCodecId()));
				continue;
			}
		}
		else if (Track->GetType() == mkvparser::Track::kAudio)
		{
			if (GetCodecFromMKVTrack(Track->GetCodecId()) != ETrackCodec::Undefined)
			{
				AudioTracks.Add(static_cast<const mkvparser::AudioTrack*>(Track));
			}
			else
			{
				UE_LOGF(LogWebMMedia, Warning, "File contains unsupported audio track %d: %ls", i, UTF8_TO_TCHAR(Track->GetCodecId()));
				continue;
			}
		}
		else
		{
			UE_LOGF(LogWebMMedia, Warning, "File contains unsupported track %d: %ls", i, UTF8_TO_TCHAR(Track->GetCodecId()));
			continue;
		}
	}

	if (VideoTracks.IsEmpty() && AudioTracks.IsEmpty())
	{
		UE_LOGF(LogWebMMedia, Warning, "File doesn't have usable video or audio.");
		return false;
	}
	return true;
}

void FWebMMediaPlayer::MkvSeekToNextValidBlock()
{
	while(1)
	{
		if (!MkvCurrentCluster)
		{
			if (!MkvSegment)
			{
				bFileError = true;
				return;
			}
			MkvCurrentCluster = MkvSegment->GetFirst();
			MkvCurrentBlockEntry = nullptr;
			if (!MkvCurrentCluster)
			{
				bFileError = true;
				return;
			}
		}

		if (!MkvCurrentBlockEntry || MkvCurrentBlockEntry->EOS())
		{
			if (MkvCurrentCluster->GetFirst(MkvCurrentBlockEntry) != 0)
			{
				UE_LOGF(LogWebMMedia, Warning, "Something went wrong while seeking");
				bFileError = true;
				return;
			}
		}
		else
		{
			if (MkvCurrentCluster->GetNext(MkvCurrentBlockEntry, MkvCurrentBlockEntry) != 0)
			{
				UE_LOGF(LogWebMMedia, Warning, "Something went wrong while seeking");
				bFileError = true;
				return;
			}
		}

		if (!MkvCurrentBlockEntry || MkvCurrentBlockEntry->EOS())
		{
			MkvCurrentBlockEntry = nullptr;
			MkvCurrentCluster = MkvSegment->GetNext(MkvCurrentCluster);
			if (!MkvCurrentCluster)
			{
				bFileError = true;
				return;
			}
			if (MkvCurrentCluster->EOS())
			{
				bFileReachedEOS = true;
				return;
			}
			else
			{
				continue;
			}
		}
		return;
	}
}

void FWebMMediaPlayer::MkvSeekToTime(const FTimespan& InTime)
{
	if (MkvSegment)
	{
		uint64 TimeInNs = (uint64)InTime.GetTotalMicroseconds() * 1000U;
		MkvCurrentCluster = MkvSegment->FindCluster(TimeInNs);
		MkvCurrentBlockEntry = nullptr;
	}
}

void FWebMMediaPlayer::Resume()
{
	CurrentState = EMediaState::Playing;
	OutEvents.Push(EMediaEvent::PlaybackResumed);
}

void FWebMMediaPlayer::Pause()
{
	CurrentState = EMediaState::Paused;
	OutEvents.Push(EMediaEvent::PlaybackSuspended);
}

void FWebMMediaPlayer::AddVideoSampleFromDecodingThread(TSharedRef<FWebMMediaTextureSample, ESPMode::ThreadSafe> InSample)
{
	if (InSample->GetDecoderIndex() == CurrentDecoderIndex)
	{
		InSample->SetShutdownPoolableDelegate(VideoFrameDoneDelegate);
		FScopeLock lock(&DecodedSamplesLock);
		DecodedVideoFrames.Emplace(InSample);
	}
}

void FWebMMediaPlayer::AddAudioSampleFromDecodingThread(TSharedRef<FWebMMediaAudioSample, ESPMode::ThreadSafe> InSample)
{
	if (InSample->GetDecoderIndex() == CurrentDecoderIndex)
	{
		InSample->SetShutdownPoolableDelegate(AudioFrameDoneDelegate);
		FScopeLock lock(&DecodedSamplesLock);
		DecodedAudioFrames.Emplace(InSample);
	}
}

void FWebMMediaPlayer::ReportVideoDecodingError(FString InErrorMessage)
{
	bDecodeError = true;
}

void FWebMMediaPlayer::ReportAudioDecodingError(FString InErrorMessage)
{
	bDecodeError = true;
}


void FWebMMediaPlayer::ReturnVideoSample(const FWebMMediaTextureSample* InSample)
{
	if (InSample->GetDecoderIndex() == CurrentDecoderIndex)
	{
		FScopeLock lock(&TimeLock);
		CurrentVideoTime = InSample->GetTime().Time;
	}
}

void FWebMMediaPlayer::ReturnAudioSample(const FWebMMediaAudioSample* InSample)
{
	if (InSample->GetDecoderIndex() == CurrentDecoderIndex)
	{
		FScopeLock lock(&TimeLock);
		CurrentAudioTime = InSample->GetTime().Time;
	}
}


#undef LOCTEXT_NAMESPACE

#endif // WITH_WEBM_LIBS
