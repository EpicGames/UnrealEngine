// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBM_LIBS

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"
#include "Misc/Guid.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "IMediaEventSink.h"
#include "Misc/Timespan.h"
#include "Misc/FrameRate.h"
#include "MkvFileReader.h"
#include "WebMSamplesSink.h"
#include "WebMMediaTextureSample.h"
#include "WebMMediaAudioSample.h"
#include <atomic>


class FMediaSamples;
class FWebMVideoDecoder;
class FWebMAudioDecoder;
struct FWebMFrame;

class FWebMMediaPlayer
	: public IMediaPlayer
	, public IWebMSamplesSink
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaTracks
	, protected IMediaView
{
public:
	FWebMMediaPlayer(IMediaEventSink& InEventSink);
	virtual ~FWebMMediaPlayer();

public:
	//~ IWebMSamplesSink interface
	virtual void AddVideoSampleFromDecodingThread(TSharedRef<FWebMMediaTextureSample, ESPMode::ThreadSafe> Sample) override;
	virtual void AddAudioSampleFromDecodingThread(TSharedRef<FWebMMediaAudioSample, ESPMode::ThreadSafe> Sample) override;
	virtual void ReportVideoDecodingError(FString InErrorMessage) override;
	virtual void ReportAudioDecodingError(FString InErrorMessage) override;

public:
	//~ IMediaPlayer interface
	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;

protected:
	//~ IMediaControls interface
	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override
	{ check(!"You have to call the override with additional options!"); return false; }
	virtual bool Seek(const FTimespan& InNewTime, const FMediaSeekParams& InAdditionalParams) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;
	virtual bool GetPlayerFeatureFlag(EFeatureFlag InFlag) const override;
	virtual bool FlushOnSeekStarted() const override;
	virtual bool FlushOnSeekCompleted() const override;

protected:
	//~ IMediaTracks interface
	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;

private:
	enum class ETrackCodec
	{
		Undefined,
		VP8,
		VP9,
		Vorbis,
		Opus
	};

	struct FSeekTo
	{
		FTimespan TargetTime;
		bool bAwaitTime = false;
		bool bGotVideoKeyframe = false;
		void Reset()
		{
			TargetTime = FTimespan::Zero();
			bAwaitTime = false;
			bGotVideoKeyframe = false;
		}
	};

	static ETrackCodec GetCodecFromMKVTrack(const char* InMKVCodecName);
	static FString GetMKVCodec(ETrackCodec InCodec);
	void TickInternal();
	void ReturnVideoSample(const FWebMMediaTextureSample* InSample);
	void ReturnAudioSample(const FWebMMediaAudioSample* InSample);
	void Resume();
	void Pause();
	bool MkvRead();
	void MkvSeekToNextValidBlock();
	void MkvSeekToTime(const FTimespan& Time);


	IMediaEventSink& EventSink;
	FCriticalSection MediaSamplesLock;
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;
	TArray<EMediaEvent> OutEvents;
	uint32 CurrentDecoderIndex { 0 };
	FString MediaUrl;
	TArray<const mkvparser::VideoTrack*> VideoTracks;
	TArray<const mkvparser::AudioTrack*> AudioTracks;
	TUniquePtr<FWebMVideoDecoder> VideoDecoder;
	TUniquePtr<FWebMAudioDecoder> AudioDecoder;
	TUniquePtr<FMkvFileReader> MkvReader;
	TUniquePtr<mkvparser::Segment> MkvSegment;
	const mkvparser::Cluster* MkvCurrentCluster { };
	const mkvparser::BlockEntry* MkvCurrentBlockEntry { };
	EMediaState CurrentState { EMediaState::Closed };
	TOptional<ETrackCodec> NewPendingVideoCodec;
	TOptional<ETrackCodec> NewPendingAudioCodec;
	int32 SelectedVideoTrack { INDEX_NONE };
	int32 SelectedAudioTrack { INDEX_NONE };
	bool bPendingVideoTrackChange { false };
	bool bPendingAudioTrackChange { false };
	FTimespan CurrentTime;
	TOptional<FTimespan> SeekToTime;
	FSeekTo SeekTo;

	int32 CurrentSeekIndex { 0 };
	int32 CurrentLoopIndex { 0 };
	bool bFileReachedEOS { false };
	bool bFileError { false };
	bool bDecodeError { false };
	bool bLooping { false} ;

	TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> VideoFramesReadyForDecode;
	TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> AudioFramesReadyForDecode;
	TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> LoadedVideoFrames;
	TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>> LoadedAudioFrames;
	FTimespan MovieDuration;
	FFrameRate VideoFrameRate {0,0};
	bool bRecalculateFrameRate = false;
	FCriticalSection TimeLock;
	TOptional<FTimespan> CurrentVideoTime;
	TOptional<FTimespan> CurrentAudioTime;

	FCriticalSection DecodedSamplesLock;
	std::atomic<int32> NumDecodingEnqueuedVideoFrames;
	std::atomic<int32> NumDecodingEnqueuedAudioFrames;
	TArray<TSharedPtr<FWebMMediaTextureSample, ESPMode::ThreadSafe>> DecodedVideoFrames;
	TArray<TSharedPtr<FWebMMediaAudioSample, ESPMode::ThreadSafe>> DecodedAudioFrames;
	FWebMMediaTextureSample::FShutdownPoolableDlg VideoFrameDoneDelegate;
	FWebMMediaAudioSample::FShutdownPoolableDlg AudioFrameDoneDelegate;
};

#endif // WITH_WEBM_LIBS
