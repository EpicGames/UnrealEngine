// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMMediaTextureSample.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "PipelineStateCache.h"
#include "MediaShaders.h"

FWebMMediaTextureSample::FWebMMediaTextureSample()
	: Time(FTimespan::Zero())
{
}

void FWebMMediaTextureSample::Initialize(const FSourceImageDesc& InSourcePlanes, FIntPoint InDisplaySize, FMediaTimeStamp InTime, FTimespan InDuration, uint32 InDecoderIndex)
{
	bFormatChanged = ImagePlanes != InSourcePlanes;
	ImagePlanes = InSourcePlanes;
	if (bFormatChanged)
	{
		ImagePlanes.Reset();
	}
	for(int32 i=0; i<4; ++i)
	{
		if (InSourcePlanes.SrcData[i])
		{
			if (!ImagePlanes.Data[i])
			{
				// The stride already includes the bytes per pixels, so no need to factor that in again.
				ImagePlanes.AllocatedDataSize[i] = ImagePlanes.Stride[i] * ImagePlanes.Dimensions[i].Y;
				ImagePlanes.Data[i] = FMemory::Malloc(ImagePlanes.AllocatedDataSize[i]);
			}
			if (ImagePlanes.Data[i])
			{
				FMemory::Memcpy(ImagePlanes.Data[i], InSourcePlanes.SrcData[i], ImagePlanes.AllocatedDataSize[i]);
			}
		}
	}
	Time = InTime;
	DisplaySize = InDisplaySize;
	Duration = InDuration;
	DecoderIndex = InDecoderIndex;
}

const void* FWebMMediaTextureSample::GetBuffer()
{
	return nullptr;
}

FIntPoint FWebMMediaTextureSample::GetDim() const
{
	return ImagePlanes.Dimensions[0];
}

FTimespan FWebMMediaTextureSample::GetDuration() const
{
	return Duration;
}

EMediaTextureSampleFormat FWebMMediaTextureSample::GetFormat() const
{
	return ImagePlanes.BytesPerPixel[0]==1 ? EMediaTextureSampleFormat::CharBGRA : EMediaTextureSampleFormat::RGBA16;
}

FIntPoint FWebMMediaTextureSample::GetOutputDim() const
{
	return DisplaySize;
}

uint32 FWebMMediaTextureSample::GetStride() const
{
	if (!Texture.IsValid())
	{
		return 0;
	}
	return Texture->GetSizeX() * 4 * ImagePlanes.BytesPerPixel[0];
}

FRHITexture* FWebMMediaTextureSample::GetTexture() const
{
	return Texture.GetReference();
}

FMediaTimeStamp FWebMMediaTextureSample::GetTime() const
{
	return FMediaTimeStamp(Time);
}

bool FWebMMediaTextureSample::IsCacheable() const
{
	return true;
}

bool FWebMMediaTextureSample::IsOutputSrgb() const
{
	return true;
}

void FWebMMediaTextureSample::ShutdownPoolable()
{
	ShutdownDelegate.ExecuteIfBound(this);

	// Drop reference to the texture. It should be released by the outside system.
	Texture = nullptr;

	Time = FMediaTimeStamp(FTimespan::Zero());
}

TRefCountPtr<FRHITexture> FWebMMediaTextureSample::GetTextureRef() const
{
	return Texture;
}

IMediaTextureSampleConverter* FWebMMediaTextureSample::GetMediaTextureSampleConverter()
{
	return this;
}

uint32 FWebMMediaTextureSample::GetConverterInfoFlags() const
{
	return IMediaTextureSampleConverter::ConverterInfoFlags_WillCreateOutputTexture;
}

bool FWebMMediaTextureSample::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
{
	// Create or recreate the Y,U and V planes for conversion to RGBA if the format changed.
	if (bFormatChanged || !ImagePlanes.SrcPlaneTexture[0].IsValid())
	{
		for(int32 i=0; i<4; ++i)
		{
			if (ImagePlanes.Data[i])
			{
				const FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(TEXT("FWebMMediaTextureSample_Cnv"), ImagePlanes.Dimensions[i].X, ImagePlanes.Dimensions[i].Y, ImagePlanes.BytesPerPixel[i] == 1 ? PF_G8 : PF_G16);
				ImagePlanes.SrcPlaneTexture[i] = RHICmdList.CreateTexture(Desc);
				if (!ImagePlanes.SrcPlaneTexture[i].IsValid())
				{
					return false;
				}
			}
			else
			{
				ImagePlanes.SrcPlaneTexture[i].SafeRelease();
			}
		}
	}

	// Copy the plane data across
	for(int32 i=0; i<4; ++i)
	{
		if (ImagePlanes.Data[i])
		{
			uint32 Stride = 0;
			uint8* TextureMemory = (uint8*)RHICmdList.LockTexture2D(ImagePlanes.SrcPlaneTexture[i].GetReference(), 0, RLM_WriteOnly, Stride, false);
			if (TextureMemory)
			{
				const uint8* SrcData = (const uint8*)ImagePlanes.Data[i];
				const uint32 SrcNB = ImagePlanes.BytesPerPixel[i] * ImagePlanes.Dimensions[i].X;
				const uint32 SrcStride = ImagePlanes.Stride[i];	// The source stride is already measured in bytes-per-pixel, so do not factor this in again!
				for(int32 y=0,yMax=ImagePlanes.Dimensions[i].Y; y<yMax; ++y)
				{
					FMemory::Memcpy(TextureMemory, SrcData, SrcNB);
					SrcData += SrcStride;
					TextureMemory += Stride;
				}
				RHICmdList.UnlockTexture2D(ImagePlanes.SrcPlaneTexture[i].GetReference(), 0, false);
			}
			else
			{
				return false;
			}
		}
	}

	// Create the RGBA texture
	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("FWebMMediaTextureSample"))
		.SetExtent(ImagePlanes.Dimensions[0])
		.SetFormat(ImagePlanes.BytesPerPixel[0] == 1 ? PF_B8G8R8A8 : PF_A16B16G16R16)
		.SetFlags(ETextureCreateFlags::SRGB | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetInitialState(ERHIAccess::SRVMask);
	Texture = RHICmdList.CreateTexture(Desc);

	FRHITexture* RenderTarget = GetTexture();
	if (!RenderTarget)
	{
		return false;
	}
	RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::SRVGraphics, ERHIAccess::RTV));
	FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::Load_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertYUVtoRGBA"));
	{
		// configure media shaders
		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		TShaderMapRef<FYUVScaledConvertPS> PixelShader(ShaderMap);
		TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		{
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;
		}

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParametersLegacyPS(RHICmdList, PixelShader,
			ImagePlanes.SrcPlaneTexture[0]->GetTexture2D(),
			ImagePlanes.SrcPlaneTexture[1]->GetTexture2D(),
			ImagePlanes.SrcPlaneTexture[2]->GetTexture2D(),
			ImagePlanes.Dimensions[0],
			ImagePlanes.VpxColorSpace == 5 ? (ImagePlanes.VpxFullRange ? MediaShaders::YuvToRgbRec2020Unscaled : MediaShaders::YuvToRgbRec2020Scaled) :
				ImagePlanes.VpxColorSpace == 1 ? (ImagePlanes.VpxFullRange ? MediaShaders::YuvToRgbRec601Unscaled : MediaShaders::YuvToRgbRec601Scaled) :
				(ImagePlanes.VpxFullRange ? MediaShaders::YuvToRgbRec709Unscaled : MediaShaders::YuvToRgbRec709Scaled),
			ImagePlanes.BytesPerPixel[0] == 1 ? (ImagePlanes.VpxFullRange ? MediaShaders::YUVOffsetNoScale8bits : MediaShaders::YUVOffset8bits) : (ImagePlanes.VpxFullRange ? MediaShaders::YUVOffsetNoScale10bits : MediaShaders::YUVOffset10bits),
			true,
			ImagePlanes.BytesPerPixel[0] == 1 ? 1.0f : 64.0f);

		// draw full-size quad
		RHICmdList.SetViewport(0, 0, 0.0f, ImagePlanes.Dimensions[0].X, ImagePlanes.Dimensions[0].Y, 1.0f);
		FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
		RHICmdList.SetStreamSource(0, VertexBuffer, 0);
		RHICmdList.DrawPrimitive(0, 2, 1);
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::RTV, ERHIAccess::SRVGraphics));

	InDstTexture = MoveTemp(Texture);
	return true;
}

