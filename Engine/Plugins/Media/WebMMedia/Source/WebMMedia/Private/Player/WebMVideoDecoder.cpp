// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMVideoDecoder.h"

#if WITH_WEBM_LIBS

#include "WebMMediaPrivate.h"
#include "WebMMediaFrame.h"
#include "WebMMediaTextureSample.h"
#include "WebMSamplesSink.h"


FWebMVideoDecoder::FWebMVideoDecoder(IWebMSamplesSink& InSamples)
	: VideoSamplePool(new FWebMMediaTextureSamplePool)
	, Samples(InSamples)
	, Codec(ECodec::Unknown)
	, CurrentSequenceIndex(0)
	, bTexturesCreated(false)
	, bIsInitialized(false)
{
}

FWebMVideoDecoder::~FWebMVideoDecoder()
{
	Close();
}

bool FWebMVideoDecoder::Initialize(const char* CodecName)
{
	Close();
	return InitializeInternal(FString(CodecName));
}

bool FWebMVideoDecoder::InitializeInternal(FString InCodecName)
{
	FlushPendingFrameInfos();
	const int32 NumOfThreads = 1;
	CodecConfig.threads = NumOfThreads;
	CodecConfig.w = 0;
	CodecConfig.h = 0;
	if (InCodecName.Equals(TEXT("V_VP8")))
	{
		verify(vpx_codec_dec_init(&Context, vpx_codec_vp8_dx(), &CodecConfig, /*VPX_CODEC_USE_FRAME_THREADING*/ 0) == 0);
		Codec = ECodec::VP8;
	}
	else if (InCodecName.Equals(TEXT("V_VP9")))
	{
		verify(vpx_codec_dec_init(&Context, vpx_codec_vp9_dx(), &CodecConfig, /*VPX_CODEC_USE_FRAME_THREADING*/ 0) == 0);
		Codec = ECodec::VP9;
	}
	else
	{
		UE_LOGF(LogWebMMedia, Display, "Unsupported video codec: %ls", *InCodecName);
		Codec = ECodec::Unknown;
		return false;
	}
	bIsInitialized = true;
	return true;
}

void FWebMVideoDecoder::DecodeVideoFramesAsync(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& VideoFrames)
{
	static bool bUseRenderThread = FPlatformMisc::UseRenderThread();
	if (bUseRenderThread)
	{
		FGraphEventArray Prerequisites;
		if (VideoDecodingTask)
		{
			Prerequisites.Emplace(VideoDecodingTask);
		}
		VideoDecodingTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this, VideoFrames]()
			{
				DoDecodeVideoFrames(VideoFrames);
			}, TStatId(), &Prerequisites, ENamedThreads::AnyThread);
	}
	else
	{
		DoDecodeVideoFrames(VideoFrames);
	}
}

bool FWebMVideoDecoder::IsBusy() const
{
	return VideoDecodingTask && !VideoDecodingTask->IsComplete();
}

void FWebMVideoDecoder::DoDecodeVideoFrames(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& VideoFrames)
{
	auto CreateDecoder = [&](const FString& InCodec) -> bool
	{
		if (bIsInitialized)
		{
			vpx_codec_destroy(&Context);
			FMemory::Memzero(Context);
			bIsInitialized = false;
		}
		return InitializeInternal(InCodec);
	};

	auto PullOutput = [&]() -> bool
	{
		// Unfortunately we need to make a deep copy of the decoded image.
		// libvpx only guarantees the decoded image pointer to be valid until the next call
		// of vpx_codec_decode().
		const void* ImageIter = nullptr;
		while(const vpx_image_t* Image = vpx_codec_get_frame(&Context, &ImageIter))
		{
			if (Image->fmt != vpx_img_fmt::VPX_IMG_FMT_I420 && Image->fmt != vpx_img_fmt::VPX_IMG_FMT_I42016)
			{
				UE_LOGF(LogWebMMedia, Display, "Unsupported decoded frame format");
				Samples.ReportVideoDecodingError(TEXT("Unsupported decoded frame format"));
				return false;
			}

			TSharedRef<FWebMMediaTextureSample, ESPMode::ThreadSafe> VideoSample = VideoSamplePool->AcquireShared();
			FWebMMediaTextureSample::FSourceImageDesc Src;
			Src.VpxColorSpace = Image->cs;
			Src.VpxFullRange = Image->range;
			for(int32 k=0; k<4; ++k)
			{
				if ((Src.SrcData[k] = Image->planes[k]))
				{
					Src.Stride[k] = Image->stride[k];
					Src.Dimensions[k].X = Image->d_w;
					Src.Dimensions[k].Y = Image->d_h;
					Src.BytesPerPixel[k] = (Image->bit_depth + 7) >> 3;
				}
				else
				{
					Src.Stride[k] = 0;
					Src.Dimensions[k].X = Src.Dimensions[k].Y = 0;
					Src.BytesPerPixel[k] = 0;
				}
			}
			Src.Dimensions[1].X = (1 + Src.Dimensions[1].X) >> 1;
			Src.Dimensions[1].Y = (1 + Src.Dimensions[1].Y) >> 1;
			Src.Dimensions[2].X = (1 + Src.Dimensions[2].X) >> 1;
			Src.Dimensions[2].Y = (1 + Src.Dimensions[2].Y) >> 1;

			FWebMFrameInfo* VideoFrame = (FWebMFrameInfo*)Image->user_priv;
			VideoSample->Initialize(Src, FIntPoint(Image->d_w, Image->d_h), FMediaTimeStamp(VideoFrame->Time, VideoFrame->SequenceIndex), VideoFrame->Duration, VideoFrame->DecoderIndex);
			PendingFrameInfos.Remove(VideoFrame);
			delete VideoFrame;

			// Add the sample to the queue. It will be converted lazily when needed.
			Samples.AddVideoSampleFromDecodingThread(VideoSample);
		}
		return true;
	};


	for (const TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>& VideoFrame : VideoFrames)
	{
		if (!bIsInitialized)
		{
			if (VideoFrame->CodecName.IsEmpty())
			{
				UE_LOGF(LogWebMMedia, Display, "No codec specified to initialize the codec lazily");
				Samples.ReportVideoDecodingError(TEXT("No codec specified to initialize the codec lazily"));
				return;
			}
		}
		if (VideoFrame->CodecName.Len())
		{
			if (!CreateDecoder(VideoFrame->CodecName))
			{
				UE_LOGF(LogWebMMedia, Display, "Failed to create codec lazily");
				Samples.ReportVideoDecodingError(TEXT("Failed to create codec lazily"));
				return;
			}
		}

		// Change in sequence?
		if (VideoFrame->SequenceIndex != CurrentSequenceIndex)
		{
			// Flush pending frames.
			if (bIsInitialized)
			{
				vpx_codec_err_t Result = vpx_codec_decode(&Context, nullptr, 0, nullptr, 0);
				PullOutput();
			}
			if (!CreateDecoder(Codec == ECodec::VP8 ? TEXT("V_VP8"): TEXT("V_VP9")))
			{
				Samples.ReportVideoDecodingError(TEXT("Failed to reset codec"));
				return;
			}
			CurrentSequenceIndex = VideoFrame->SequenceIndex;
		}

		// Create a structure with the frame information
		FWebMFrameInfo* FrameInf = new FWebMFrameInfo(*VideoFrame);
		PendingFrameInfos.Emplace(FrameInf);
		if (vpx_codec_decode(&Context, VideoFrame->Data.GetData(), VideoFrame->Data.Num(), FrameInf, 0) != 0)
		{
			PendingFrameInfos.Remove(FrameInf);
			delete FrameInf;
			FString Msg = FString::Printf(TEXT("Error decoding video frame (%s)"), ANSI_TO_TCHAR(vpx_codec_error(&Context)));
			UE_LOGF(LogWebMMedia, Display, "%ls", *Msg);
			Samples.ReportVideoDecodingError(Msg);
			return;
		}

		PullOutput();
	}
}

void FWebMVideoDecoder::FlushPendingFrameInfos()
{
	while(!PendingFrameInfos.IsEmpty())
	{
		delete PendingFrameInfos.Pop();
	}
}

void FWebMVideoDecoder::Close()
{
	if (VideoDecodingTask && !VideoDecodingTask->IsComplete())
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(VideoDecodingTask);
	}

	// Make sure all compute shader decoding is done
	//
	// This function can also be called on a rendering thread (the streamer is ticked there during a startup movie, and decoder gets deleted on StartNextMovie()
	// if there are >1 movie queued). In this case we will ensure that the resources survive for one more frame after use by other means.
	if (IsInGameThread())
	{
		FlushRenderingCommands();
	}

	if (bIsInitialized)
	{
		vpx_codec_destroy(&Context);
		bIsInitialized = false;
	}
	FlushPendingFrameInfos();

	bTexturesCreated = false;
}

#endif // WITH_WEBM_LIBS
