// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMContainer.h"

#if WITH_WEBM_LIBS

#include "Player/MkvFileReader.h"
#include "WebMMediaPrivate.h"
#include "WebMMediaFrame.h"

struct FWebMContainer::FMkvFileState
{
	TArray<const mkvparser::VideoTrack*> VideoTracks;
	TArray<const mkvparser::AudioTrack*> AudioTracks;
	TUniquePtr<mkvparser::Segment> CurrentSegment;
	const mkvparser::Cluster* CurrentCluster;
	const mkvparser::BlockEntry* CurrentBlockEntry;
};

FWebMContainer::FWebMContainer()
	: MkvFile(new FMkvFileState())
	, SelectedAudioTrack(INDEX_NONE)
	, SelectedVideoTrack(INDEX_NONE)
	, bNoMoreToRead(false)
{
	MkvFile->CurrentSegment = nullptr;
	MkvFile->CurrentBlockEntry = nullptr;
}

FWebMContainer::~FWebMContainer()
{
}

bool FWebMContainer::Open(const FString& FilePath)
{
	MkvReader.Reset(new FMkvFileReader());
	if (!MkvReader->Open(*FilePath))
	{
		UE_LOGF(LogWebMMedia, Error, "Failed opening video file: %ls", *FilePath);
		return false;
	}

	int64 FilePosition = 0;
	if (mkvparser::EBMLHeader().Parse(MkvReader.Get(), FilePosition) != 0)
	{
		UE_LOGF(LogWebMMedia, Error, "Failed parsing MKV header");
		return false;
	}

	mkvparser::Segment* Segment;
	if (mkvparser::Segment::CreateInstance(MkvReader.Get(), FilePosition, Segment) != 0)
	{
		UE_LOGF(LogWebMMedia, Error, "Failed parsing MKV");
		return false;
	}

	MkvFile->CurrentSegment.Reset(Segment);

	if (MkvFile->CurrentSegment->Load() < 0)
	{
		UE_LOGF(LogWebMMedia, Error, "Failed loading MKV header");
		return false;
	}

	// Read all the tracks

	const mkvparser::Tracks* Tracks = MkvFile->CurrentSegment->GetTracks();
	int32 NumOfTracks = Tracks->GetTracksCount();

	if (NumOfTracks == 0)
	{
		UE_LOGF(LogWebMMedia, Warning, "File doesn't have any tracks");
		return false;
	}

	for (int32 i = 0; i < NumOfTracks; ++i)
	{
		const mkvparser::Track* Track = Tracks->GetTrackByIndex(i);
		check(Track);

		if (Track->GetType() == mkvparser::Track::kVideo)
		{
			if (FCStringAnsi::Strcmp(Track->GetCodecId(), "V_VP9") == 0 || FCStringAnsi::Strcmp(Track->GetCodecId(), "V_VP8") == 0)
			{
				MkvFile->VideoTracks.Add(static_cast<const mkvparser::VideoTrack*>(Track));

				// Set as default
				if (SelectedVideoTrack == INDEX_NONE)
				{
					SelectedVideoTrack = MkvFile->VideoTracks.Num() - 1;
				}

			}
			else
			{
				UE_LOGF(LogWebMMedia, Warning, "File contains unsupported video track %d: %ls", i, UTF8_TO_TCHAR(Track->GetCodecId()));
				continue;
			}
		}
		else if (Track->GetType() == mkvparser::Track::kAudio)
		{
			if (FCStringAnsi::Strcmp(Track->GetCodecId(), "A_OPUS") == 0 || FCStringAnsi::Strcmp(Track->GetCodecId(), "A_VORBIS") == 0)
			{
				MkvFile->AudioTracks.Add(static_cast<const mkvparser::AudioTrack*>(Track));

				// Set as default
				if (SelectedAudioTrack == INDEX_NONE)
				{
					SelectedAudioTrack = MkvFile->AudioTracks.Num() - 1;
				}
			}
			else
			{
				UE_LOGF(LogWebMMedia, Warning, "File contains unsupported audio track %d: %ls", i, UTF8_TO_TCHAR(Track->GetCodecId()));
				continue;
			}
		}
		else
		{
			UE_LOGF(LogWebMMedia, Warning, "File contains unsupported track %d: %ls", i, UTF8_TO_TCHAR(Track->GetCodecId()));
			continue;
		}
	}

	if (MkvFile->VideoTracks.Num() == 0 || MkvFile->AudioTracks.Num() == 0)
	{
		UE_LOGF(LogWebMMedia, Warning, "File doesn't have video or audio. Right now only files with both are supported");
		return false;
	}

	CurrentTime = 0;

	return true;
}

void FWebMContainer::ReadFrames(FTimespan ReadBufferLength, TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& AudioFrames, TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& VideoFrames)
{
	if (bNoMoreToRead)
	{
		return;
	}

	FTimespan UpToReadTime = CurrentTime + ReadBufferLength;

	while (CurrentTime < UpToReadTime)
	{
		SeekToNextValidBlock();

		if (bNoMoreToRead)
		{
			break;
		}

		CurrentTime = FTimespan::FromMicroseconds(MkvFile->CurrentBlockEntry->GetBlock()->GetTime(MkvFile->CurrentCluster) / 1000.0);

		for (int32 i = 0; i < MkvFile->CurrentBlockEntry->GetBlock()->GetFrameCount(); ++i)
		{
			const mkvparser::Block::Frame& MkvFrame = MkvFile->CurrentBlockEntry->GetBlock()->GetFrame(i);

			TSharedPtr<FWebMFrame, ESPMode::ThreadSafe> Frame = MakeShared<FWebMFrame>();
			Frame->Time = CurrentTime;
			Frame->Duration = FMkvFileReader::GetVideoFrameDuration(*MkvFile->VideoTracks[SelectedVideoTrack]);
			Frame->Data.SetNumUninitialized(MkvFrame.len);
			MkvFrame.Read(MkvReader.Get(), Frame->Data.GetData());

			int32 TrackNumber = MkvFile->CurrentBlockEntry->GetBlock()->GetTrackNumber();

			if (SelectedVideoTrack != INDEX_NONE && MkvFile->VideoTracks[SelectedVideoTrack]->GetNumber() == TrackNumber)
			{
				VideoFrames.Add(Frame);
				CurrentTime += FTimespan::FromSeconds(1.0 / MkvFile->VideoTracks[SelectedVideoTrack]->GetFrameRate());
			}
			else if (SelectedAudioTrack != INDEX_NONE && MkvFile->AudioTracks[SelectedAudioTrack]->GetNumber() == TrackNumber)
			{
				AudioFrames.Add(Frame);
			}
		}
	}
}

FWebMAudioTrackInfo FWebMContainer::GetCurrentAudioTrackInfo() const
{
	FWebMAudioTrackInfo Info;
	if (SelectedAudioTrack == INDEX_NONE)
	{
		Info.bIsValid = false;
		return Info;
	}

	const mkvparser::AudioTrack* AudioTrack = MkvFile->AudioTracks[SelectedAudioTrack];

	Info.bIsValid = true;
	Info.CodecName = AudioTrack->GetCodecId();
	Info.CodecPrivateData = AudioTrack->GetCodecPrivate(Info.CodecPrivateDataSize);
	Info.SampleRate = AudioTrack->GetSamplingRate();
	Info.NumOfChannels = AudioTrack->GetChannels();

	return Info;
}

FWebMVideoTrackInfo FWebMContainer::GetCurrentVideoTrackInfo() const
{
	FWebMVideoTrackInfo Info;
	if (SelectedVideoTrack == INDEX_NONE)
	{
		Info.bIsValid = false;
		return Info;
	}

	const mkvparser::VideoTrack* VideoTrack = MkvFile->VideoTracks[SelectedVideoTrack];

	Info.bIsValid = true;
	Info.CodecName = VideoTrack->GetCodecId();

	return Info;
}

void FWebMContainer::SeekToNextValidBlock()
{
	for (;;)
	{
		if (!MkvFile->CurrentCluster)
		{
			MkvFile->CurrentCluster = MkvFile->CurrentSegment->GetFirst();
			MkvFile->CurrentBlockEntry = nullptr;
			check(MkvFile->CurrentCluster);
		}

		if (!MkvFile->CurrentBlockEntry || MkvFile->CurrentBlockEntry->EOS())
		{
			if (MkvFile->CurrentCluster->GetFirst(MkvFile->CurrentBlockEntry) != 0)
			{
				UE_LOGF(LogWebMMedia, Warning, "Something went wrong while seeking");
				bNoMoreToRead = true;
				return;
			}
		}
		else
		{
			if (MkvFile->CurrentCluster->GetNext(MkvFile->CurrentBlockEntry, MkvFile->CurrentBlockEntry) != 0)
			{
				UE_LOGF(LogWebMMedia, Warning, "Something went wrong while seeking");
				bNoMoreToRead = true;
				return;
			}
		}

		if (!MkvFile->CurrentBlockEntry || MkvFile->CurrentBlockEntry->EOS())
		{
			MkvFile->CurrentBlockEntry = nullptr;
			MkvFile->CurrentCluster = MkvFile->CurrentSegment->GetNext(MkvFile->CurrentCluster);
			check(MkvFile->CurrentCluster);

			if (MkvFile->CurrentCluster->EOS())
			{
				bNoMoreToRead = true;
				return;
			}
			else
			{
				continue;
			}
		}

		return;
	}
}

#endif // WITH_WEBM_LIBS
