// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMMediaPrivate.h"

#include "Modules/ModuleManager.h"

#include "IWebMMediaModule.h"
#if WITH_WEBM_LIBS
#include "Player/WebMMediaPlayer.h"
#endif // WITH_WEBM_LIBS


DEFINE_LOG_CATEGORY(LogWebMMedia);


/**
 * Implements the WebMMedia module.
 */
class FWebMMediaModule : public IWebMMediaModule
{
public:

	//~ IWebMMediaModule interface

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
#if WITH_WEBM_LIBS
		return MakeShared<FWebMMediaPlayer, ESPMode::ThreadSafe>(EventSink);
#else
		return nullptr;
#endif
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FWebMMediaModule, WebMMedia)
