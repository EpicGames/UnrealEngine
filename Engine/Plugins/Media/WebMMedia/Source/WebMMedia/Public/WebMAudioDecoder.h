// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#define UE_API WEBMMEDIA_API

class IWebMSamplesSink;
class FWebMMediaAudioSample;
class FWebMMediaAudioSamplePool;
struct FWebMFrame;
struct OpusDecoder;

class FWebMAudioDecoder
{
public:
	UE_API FWebMAudioDecoder(IWebMSamplesSink& InSamples);
	UE_API ~FWebMAudioDecoder();

public:
	UE_API bool Initialize(const char* InCodec, int32 InSampleRate, int32 InChannels, const uint8* CodecPrivateData, size_t CodecPrivateDataSize);
	UE_API void DecodeAudioFramesAsync(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& AudioFrames);
	UE_API bool IsBusy() const;

private:
	enum ECodec
	{
		Undefined,
		Opus,
		Vorbis,
	};

	struct FVorbisDecoder;
	TArray<uint8> CodecSpecificData;
	TUniquePtr<FWebMMediaAudioSamplePool> AudioSamplePool;
	TUniquePtr<FVorbisDecoder> VorbisDecoder;
	FGraphEventRef AudioDecodingTask;
	TArray<uint8> DecodeBuffer;
	IWebMSamplesSink& Samples;
	OpusDecoder* OpusDecoder { };
	ECodec Codec { ECodec::Undefined };
	int64 CurrentSequenceIndex { };
	int32 FrameSize { };
	int32 SampleRate { };
	int32 Channels { };

	bool InitializeInternal(FString InCodec, int32 InSampleRate, int32 InChannels, TConstArrayView<uint8> InCodecPrivate);
	void InitializeOpus();
	bool InitializeVorbis(const uint8* CodecPrivateData, size_t CodecPrivateDataSize);
	void DoDecodeAudioFrames(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& AudioFrames);
	int32 DecodeOpus(const TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>& AudioFrame);
	int32 DecodeVorbis(const TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>& AudioFrame);
};

#undef UE_API
