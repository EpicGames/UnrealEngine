// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

#define UE_API WEBMMEDIA_API

struct FWebMFrame;

struct FWebMAudioTrackInfo
{
	const char* CodecName;
	const uint8* CodecPrivateData;
	size_t CodecPrivateDataSize;
	int32 SampleRate;
	int32 NumOfChannels;
	bool bIsValid;
};

struct FWebMVideoTrackInfo
{
	const char* CodecName;
	bool bIsValid;
};

class FMkvFileReader;

class FWebMContainer
{
public:
	UE_API FWebMContainer();
	UE_API virtual ~FWebMContainer();

	UE_API bool Open(const FString& File);
	UE_API void ReadFrames(FTimespan AmountOfTimeToRead, TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& AudioFrames, TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& VideoFrames);
	UE_API FWebMAudioTrackInfo GetCurrentAudioTrackInfo() const;
	UE_API FWebMVideoTrackInfo GetCurrentVideoTrackInfo() const;

private:
	struct FMkvFileState;
	TUniquePtr<FMkvFileReader> MkvReader;
	TUniquePtr<FMkvFileState> MkvFile;
	FTimespan CurrentTime;
	int32 SelectedAudioTrack;
	int32 SelectedVideoTrack;
	bool bNoMoreToRead;

	void SeekToNextValidBlock();
};

#undef UE_API
