// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FWebMFrameInfo
{
	FTimespan Time;
	FTimespan Duration;
	int64 SequenceIndex = 0;
	uint32 DecoderIndex = 0;
};

struct FWebMFrame : public FWebMFrameInfo
{
	bool bIsKeyframe = false;
	TArray<uint8> Data;
	// If codec name is set then a new decoder of this format is needed.
	FString CodecName;
	TArray<uint8> CodecSpecificData;
	int32 NumChannels = 0;
	int32 SamplingRate = 0;
};
