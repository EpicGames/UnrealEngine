// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaTextureSample.h"
#include "IMediaTextureSampleConverter.h"
#include "MediaObjectPool.h"
#include "RHI.h"
#include "RHIUtilities.h"

#define UE_API WEBMMEDIA_API

class FRHITexture;

class FWebMMediaTextureSample
	: public IMediaTextureSample
	, public IMediaTextureSampleConverter
	, public IMediaPoolable
{
public:
	UE_API FWebMMediaTextureSample();

	struct FImagePlaneDesc
	{
		// 0=Y, 1=U, 2=V, 3=Alpha (not used)
		FIntPoint Dimensions[4];
		uint32 Stride[4] {};
		uint8 BytesPerPixel[4] {1,1,1,1};
		uint8 VpxColorSpace {2};
		uint8 VpxFullRange {0};
		bool operator==(const FImagePlaneDesc&rhs) const
		{
			return Dimensions[0] == rhs.Dimensions[0] && Stride[0] == rhs.Stride[0] && BytesPerPixel[0] == rhs.BytesPerPixel[0] &&
				   Dimensions[1] == rhs.Dimensions[1] && Stride[1] == rhs.Stride[1] && BytesPerPixel[1] == rhs.BytesPerPixel[1] &&
				   Dimensions[2] == rhs.Dimensions[2] && Stride[2] == rhs.Stride[2] && BytesPerPixel[2] == rhs.BytesPerPixel[2] &&
				   Dimensions[3] == rhs.Dimensions[3] && Stride[3] == rhs.Stride[3] && BytesPerPixel[3] == rhs.BytesPerPixel[3] &&
				   VpxColorSpace == rhs.VpxColorSpace && VpxFullRange == rhs.VpxFullRange;
		}
		bool operator!=(const FImagePlaneDesc&rhs) const
		{ return !operator==(rhs); }
	};
	struct FSourceImageDesc : public FImagePlaneDesc
	{
		void* SrcData[4] {};
	};

	UE_API void Initialize(const FSourceImageDesc& InSourcePlanes, FIntPoint InDisplaySize, FMediaTimeStamp InTime, FTimespan InDuration, uint32 InDecoderIndex);

	//~ IMediaTextureSample interface
	UE_API const void* GetBuffer() override;
	UE_API FIntPoint GetDim() const override;
	UE_API FTimespan GetDuration() const override;
	UE_API EMediaTextureSampleFormat GetFormat() const override;
	UE_API FIntPoint GetOutputDim() const override;
	UE_API uint32 GetStride() const override;
	UE_API FRHITexture* GetTexture() const override;
	UE_API FMediaTimeStamp GetTime() const override;
	UE_API bool IsCacheable() const override;
	UE_API bool IsOutputSrgb() const override;
	UE_API IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;

	//~ IMediaTextureSampleConverter interface
	UE_API uint32 GetConverterInfoFlags() const override;
	UE_API bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;


	DECLARE_DELEGATE_OneParam(FShutdownPoolableDlg, const FWebMMediaTextureSample*);
	void SetShutdownPoolableDelegate(FShutdownPoolableDlg InDelegate)
	{ ShutdownDelegate = MoveTemp(InDelegate); }
	uint32 GetDecoderIndex() const
	{ return DecoderIndex; }

	//~ IMediaPoolable interface
	UE_API virtual void ShutdownPoolable() override;

	UE_API TRefCountPtr<FRHITexture> GetTextureRef() const;

private:
	struct FImagePlanes : public FImagePlaneDesc
	{
		FImagePlanes& operator= (const FImagePlaneDesc& rhs)
		{
			FImagePlaneDesc::operator=(rhs);
			return *this;
		}
		TRefCountPtr<FRHITexture> SrcPlaneTexture[4];
		void* Data[4] {};
		uint32 AllocatedDataSize[4] {};
		~FImagePlanes()
		{
			Reset();
		}
		void Reset()
		{
			for(int32 i=0; i<4; ++i)
			{
				if (Data[i])
				{
					FMemory::Free(Data[i]);
					Data[i] = nullptr;
				}
				AllocatedDataSize[i] = 0;
				SrcPlaneTexture[i].SafeRelease();
			}
		}
	};

	FShutdownPoolableDlg ShutdownDelegate;
	FImagePlanes ImagePlanes;
	TRefCountPtr<FRHITexture> Texture;
	FMediaTimeStamp Time;
	FTimespan Duration;
	FIntPoint DisplaySize;
	uint32 DecoderIndex = 0;
	bool bFormatChanged = true;
};

class FWebMMediaTextureSamplePool : public TMediaObjectPool<FWebMMediaTextureSample> { };

#undef UE_API
