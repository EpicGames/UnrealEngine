// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

class FWebMMediaAudioSample;
class FWebMMediaTextureSample;

class IWebMSamplesSink
{
public:
	virtual void AddVideoSampleFromDecodingThread(TSharedRef<FWebMMediaTextureSample, ESPMode::ThreadSafe> Sample) = 0;
	virtual void AddAudioSampleFromDecodingThread(TSharedRef<FWebMMediaAudioSample, ESPMode::ThreadSafe> Sample) = 0;
	virtual void ReportVideoDecodingError(FString InErrorMessage) = 0;
	virtual void ReportAudioDecodingError(FString InErrorMessage) = 0;

	virtual ~IWebMSamplesSink() {}
};
