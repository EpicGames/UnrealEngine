// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_WEBM_LIBS

#include "Templates/SharedPointer.h"
#include "MediaShaders.h"

THIRD_PARTY_INCLUDES_START
#include <vpx_decoder.h>
#include <vp8dx.h>
THIRD_PARTY_INCLUDES_END

class IWebMSamplesSink;
class FWebMMediaTextureSample;
class FWebMMediaTextureSamplePool;
struct FWebMFrameInfo;
struct FWebMFrame;

class FWebMVideoDecoder
{
public:
	WEBMMEDIA_API FWebMVideoDecoder(IWebMSamplesSink& InSamples);
	WEBMMEDIA_API ~FWebMVideoDecoder();

public:
	WEBMMEDIA_API bool Initialize(const char* CodecName);
	WEBMMEDIA_API void DecodeVideoFramesAsync(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& VideoFrames);
	WEBMMEDIA_API bool IsBusy() const;

private:
	enum class ECodec
	{
		Unknown,
		VP8,
		VP9
	};

	vpx_codec_ctx_t Context;
	vpx_codec_dec_cfg_t CodecConfig {};
	TUniquePtr<FWebMMediaTextureSamplePool> VideoSamplePool;
	TArray<FWebMFrameInfo*> PendingFrameInfos;
	FGraphEventRef VideoDecodingTask;
	IWebMSamplesSink& Samples;
	ECodec Codec;
	int64 CurrentSequenceIndex;
	bool bTexturesCreated;
	bool bIsInitialized;

	bool InitializeInternal(FString InCodecName);
	void DoDecodeVideoFrames(const TArray<TSharedPtr<FWebMFrame, ESPMode::ThreadSafe>>& VideoFrames);
	void Close();
	void FlushPendingFrameInfos();
};

#endif // WITH_WEBM_LIBS
