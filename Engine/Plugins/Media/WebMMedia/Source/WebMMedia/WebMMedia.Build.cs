// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class WebMMedia : ModuleRules
	{
		public WebMMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"WebMMediaFactory",
					"Core",
					"Engine",
					"RenderCore",
					"RHI",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Media",
					"MediaUtils",
					"libOpus",
					"UEOgg",
					"Vorbis",
				});

			if (Target.Platform == UnrealTargetPlatform.Mac ||
				Target.Platform == UnrealTargetPlatform.Win64 ||
				(Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) && Target.Architecture == UnrealArch.X64))
			{
				PublicDependencyModuleNames.Add("LibVpx");
				
				PublicDependencyModuleNames.AddRange(
					new string[] {
					"LibWebM",
				});

				PublicDefinitions.Add("WITH_WEBM_LIBS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_WEBM_LIBS=0");
			}
		}
	}
}
