// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMFileMediaSourceFactory.h"

#include "FileMediaSource.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WebMFileMediaSourceFactory)


/* UWebMPlatFileMediaSourceFactory structors
 *****************************************************************************/

UWebMPlatFileMediaSourceFactory::UWebMPlatFileMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// add formats we support
	Formats.Add(TEXT("webm;WebM Multimedia File"));

	SupportedClass = UFileMediaSource::StaticClass();

	bEditorImport = true;
}


/* UFactory interface
 *****************************************************************************/

bool UWebMPlatFileMediaSourceFactory::FactoryCanImport(const FString& Filename)
{
	return true;
}

UObject* UWebMPlatFileMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UFileMediaSource* MediaSource = NewObject<UFileMediaSource>(InParent, InClass, InName, Flags);
	MediaSource->SetFilePath(CurrentFilename);

	return MediaSource;
}
