// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebMMediaFactoryPrivate.h"

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "UObject/NameTypes.h"
#include "HAL/PlatformProperties.h"

#include "WebMMedia/Public/IWebMMediaModule.h"


DEFINE_LOG_CATEGORY(LogWebMMediaFactory);

#define LOCTEXT_NAMESPACE "FWebMMediaFactoryModule"


/**
 * Implements the WebMMediaFactory module.
 */
class FWebMMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		// check file extension
		if (Scheme == TEXT("file"))
		{
			const FString Extension = FPaths::GetExtension(Location, false);

			if (!SupportedFileExtensions.Contains(Extension))
			{
				if (OutErrors != nullptr)
				{
					OutErrors->Add(FText::Format(LOCTEXT("ExtensionNotSupported", "The file extension '{0}' is not supported"), FText::FromString(Extension)));
				}

				return 0;
			}
		}

		return 80;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto WebMMediaModule = FModuleManager::LoadModulePtr<IWebMMediaModule>("WebMMedia");
		return (WebMMediaModule != nullptr) ? WebMMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "WebM Media");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("WebMMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0xdfbb4e57, 0x07dc4b4a, 0xa25b5cba, 0x0f963ac3);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return Feature == EMediaFeature::AudioSamples || Feature == EMediaFeature::VideoSamples;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported file extensions
		SupportedFileExtensions.Add(TEXT("webm"));

		// supported platforms
		SupportedPlatforms.Add(TEXT("Linux"));
		SupportedPlatforms.Add(TEXT("Mac"));
		SupportedPlatforms.Add(TEXT("Windows"));
		AddSupportedPlatform(FGuid(0x5636fbc1, 0xd2b54f62, 0xac8e7d4f, 0xb184b45a));
		SupportedPlatforms.AddUnique(FPlatformProperties::IniPlatformName());

		// supported schemes
		SupportedUriSchemes.Add(TEXT("file"));

		// register media player info
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:
	void AddSupportedPlatform(const FGuid& PlatformGuid)
	{
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");
		if (MediaModule)
		{
			FName PlatformName = MediaModule->GetPlatformName(PlatformGuid);
			if (!PlatformName.IsNone())
			{
				SupportedPlatforms.Add(PlatformName.ToString());
			}
		}
	}

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FWebMMediaFactoryModule, WebMMediaFactory);
