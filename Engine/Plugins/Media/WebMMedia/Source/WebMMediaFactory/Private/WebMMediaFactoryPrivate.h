// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

/** Log category for the the module. */
DECLARE_LOG_CATEGORY_EXTERN(LogWebMMediaFactory, Log, All);
