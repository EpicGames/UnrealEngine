// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WebMMediaFactory : ModuleRules
	{
		public WebMMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				});

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
			{
				DynamicallyLoadedModuleNames.Add("WebMMedia");
			}
		}
	}
}
