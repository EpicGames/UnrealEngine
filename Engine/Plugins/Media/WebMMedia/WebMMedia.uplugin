{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "WebM Video Player",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WebMMedia",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"TargetDenyList": [
				"Server"
			]
		},
		{
			"Name": "WebMMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "WebMMediaFactory",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "WebMMediaFactory",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"TargetDenyList": [
				"Server"
			]
		}
	]
}
